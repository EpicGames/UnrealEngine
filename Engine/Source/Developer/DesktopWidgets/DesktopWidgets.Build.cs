// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class DesktopWidgets : ModuleRules
{
	public DesktopWidgets(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Slate",
				"SlateCore",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
                "DesktopPlatform",
                "InputCore",
			}
		);
	}
}
