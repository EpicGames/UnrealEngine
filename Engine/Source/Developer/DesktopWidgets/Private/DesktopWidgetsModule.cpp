// Copyright Epic Games, Inc. All Rights Reserved.

#include "UObject/NameTypes.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


static const FName DesktopWidgetsTabName("DesktopWidgets");


/**
 * Implements the DesktopWidgets module.
 */
class FDesktopWidgetsModule
	: public IModuleInterface
{
public:

	// IModuleInterface interface
	
	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }
};


IMPLEMENT_MODULE(FDesktopWidgetsModule, DesktopWidgets);
