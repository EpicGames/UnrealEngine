// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SDirectoryPicker.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"
#include "DesktopPlatformModule.h"

#define LOCTEXT_NAMESPACE "SDirectoryPicker"

void SDirectoryPicker::Construct( const FArguments& InArgs )
{
	OnDirectoryChanged = InArgs._OnDirectoryChanged;

	Directory = InArgs._Directory;
	File = InArgs._File;
	Message = InArgs._Message;

	TSharedPtr<SButton> OpenButton;
	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		[
			SAssignNew(EditableTextBox, SEditableTextBox)
			.Text(this, &SDirectoryPicker::GetFilePathText)
			.OnTextChanged(this, &SDirectoryPicker::OnDirectoryTextChanged)
			.OnTextCommitted(this, &SDirectoryPicker::OnDirectoryTextCommited)
		]

		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(4.0f, 0.0f, 0.0f, 0.0f)
		[
			SAssignNew(OpenButton, SButton)
			.ToolTipText(Message)
			.OnClicked(this, &SDirectoryPicker::BrowseForDirectory)
			[
				SNew(STextBlock)
				.Text(FText::FromString(TEXT("...")))
			]
		]
	];

	OpenButton->SetEnabled(InArgs._IsEnabled);
}

void SDirectoryPicker::OnDirectoryTextChanged(const FText& InDirectoryPath)
{
	Directory = InDirectoryPath.ToString();
}

void SDirectoryPicker::OnDirectoryTextCommited(const FText& InText, ETextCommit::Type InCommitType)
{
	Directory = InText.ToString();

	OnDirectoryChanged.ExecuteIfBound(Directory);
}
	
FString SDirectoryPicker::GetFilePath() const
{
	return FPaths::Combine(*Directory, *File);
}

FText SDirectoryPicker::GetFilePathText() const
{
	return FText::FromString(GetFilePath());
}

bool SDirectoryPicker::OpenPlatformDirectoryPicker(FString& OutDirectory, const FString& DefaultPath)
{
	bool bFolderSelected = false;
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
	if ( DesktopPlatform )
	{
		void* TopWindowHandle = FSlateApplication::Get().GetActiveTopLevelWindow().IsValid() ? FSlateApplication::Get().GetActiveTopLevelWindow()->GetNativeWindow()->GetOSWindowHandle() : nullptr;

		FString FolderName;
		bFolderSelected = DesktopPlatform->OpenDirectoryDialog(
			TopWindowHandle,
			Message.ToString(),
			DefaultPath,
			FolderName
		);

		if ( bFolderSelected )
		{
			OutDirectory = FolderName;
		}
	}

	return bFolderSelected;
}

FReply SDirectoryPicker::BrowseForDirectory()
{
	if ( OpenPlatformDirectoryPicker(Directory, Directory) )
	{
		OnDirectoryChanged.ExecuteIfBound(Directory);
	}

	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE
