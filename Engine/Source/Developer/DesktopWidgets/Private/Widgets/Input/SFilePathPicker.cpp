// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/Input/SFilePathPicker.h"
#include "DesktopPlatformModule.h"
#include "Widgets/SBoxPanel.h"
#include "Framework/Application/SlateApplication.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SEditableTextBox.h"
#include "Widgets/Input/SButton.h"


#define LOCTEXT_NAMESPACE "SFilePathPicker"


/* SFilePathPicker interface
 *****************************************************************************/

void SFilePathPicker::Construct( const FArguments& InArgs )
{
	BrowseDirectory = InArgs._BrowseDirectory;
	BrowseTitle = InArgs._BrowseTitle;
	FilePath = InArgs._FilePath;
	FileTypeFilter = InArgs._FileTypeFilter;
	OnPathPicked = InArgs._OnPathPicked;
	DialogReturnsFullPath = InArgs._DialogReturnsFullPath;

	ChildSlot
	[
		SNew(SHorizontalBox)

		+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				// file path text box
				SAssignNew(TextBox, SEditableTextBox)
					.Text(this, &SFilePathPicker::HandleTextBoxText)
					.Font(InArgs._Font)
					.SelectAllTextWhenFocused(true)
					.ClearKeyboardFocusOnCommit(false)
					.OnTextCommitted(this, &SFilePathPicker::HandleTextBoxTextCommitted)
					.SelectAllTextOnCommit(false)
					.IsReadOnly(InArgs._IsReadOnly)
			]

		+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 0.0f, 0.0f, 0.0f)
			.VAlign(VAlign_Center)
			[
				// browse button
				SNew(SButton)
					.ButtonStyle(InArgs._BrowseButtonStyle)
					.ToolTipText(InArgs._BrowseButtonToolTip)
					.OnClicked(this, &SFilePathPicker::HandleBrowseButtonClicked)
					.ContentPadding(2.0f)
					.ForegroundColor(FSlateColor::UseForeground())
					.IsFocusable(false)
					[
						SNew(SImage)
							.Image(InArgs._BrowseButtonImage)
							.ColorAndOpacity(FSlateColor::UseForeground())
					]
			]
	];
}

void SFilePathPicker::SimulateTextInput(const FText& NewText)
{
	HandleTextBoxTextCommitted(NewText, ETextCommit::Default);
}

/* SFilePathPicker callbacks
 *****************************************************************************/

FReply SFilePathPicker::HandleBrowseButtonClicked()
{
	IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();

	if (DesktopPlatform == nullptr)
	{
		return FReply::Handled();
	}

	const FString DefaultPath = BrowseDirectory.IsSet()
		? BrowseDirectory.Get()
		: FPaths::GetPath(FilePath.Get());

	// show the file browse dialog
	TSharedPtr<SWindow> ParentWindow = FSlateApplication::Get().FindWidgetWindow(AsShared());
	void* ParentWindowHandle = (ParentWindow.IsValid() && ParentWindow->GetNativeWindow().IsValid())
		? ParentWindow->GetNativeWindow()->GetOSWindowHandle()
		: nullptr;

	TArray<FString> OutFiles;

	if (DesktopPlatform->OpenFileDialog(ParentWindowHandle, BrowseTitle.Get().ToString(), DefaultPath, TEXT(""), FileTypeFilter.Get(), EFileDialogFlags::None, OutFiles))
	{
		if (DialogReturnsFullPath.Get())
		{
			OnPathPicked.ExecuteIfBound(FPaths::ConvertRelativePathToFull(OutFiles[0]));
		}
		else
		{
			OnPathPicked.ExecuteIfBound(OutFiles[0]);
		}
	}

	return FReply::Handled();
}


FText SFilePathPicker::HandleTextBoxText() const
{
	return FText::FromString(FilePath.Get());
}


void SFilePathPicker::HandleTextBoxTextCommitted( const FText& NewText, ETextCommit::Type CommitInfo )
{
	OnPathPicked.ExecuteIfBound(NewText.ToString());
}


#undef LOCTEXT_NAMESPACE
