// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Application/SlateWindowHelper.h"

#define UE_API DESKTOPWIDGETS_API

class SEditableTextBox;

/**
 * A directory path box, with a button for picking a new path.
 */
class SDirectoryPicker : public SCompoundWidget
{
public:
	DECLARE_DELEGATE_OneParam(FOnDirectoryChanged, const FString& /*Directory*/);

	SLATE_BEGIN_ARGS(SDirectoryPicker)
		 : _IsEnabled(true)

		{}
		SLATE_ARGUMENT(FString, Directory)
		SLATE_ARGUMENT(FString, File)
		SLATE_ARGUMENT(FText, Message)
		SLATE_ATTRIBUTE(bool, IsEnabled)

		/** Called when a path has been picked or modified. */
		SLATE_EVENT(FOnDirectoryChanged, OnDirectoryChanged)
	SLATE_END_ARGS()
	
public:

	UE_API void Construct(const FArguments& InArgs);
	
	UE_API FString GetFilePath() const;

	FString GetDirectory() const { return Directory; }

	/**
	* Declares a delegate that is executed when a file was picked in the SFilePathPicker widget.
	*
	* The first parameter will contain the path to the picked file.
	*/
	DECLARE_DELEGATE_OneParam(FOnDirectoryPicked, const FString& /*PickedPath*/);
	
private:
	UE_API void OnDirectoryTextChanged(const FText& InDirectoryPath);
	UE_API void OnDirectoryTextCommited(const FText& InText, ETextCommit::Type InCommitType);

	UE_API FText GetFilePathText() const;

	UE_API bool OpenPlatformDirectoryPicker(FString& OutDirectory, const FString& DefaultPath);

	UE_API FReply BrowseForDirectory();

private:
	FString File;
	FString Directory;
	FText Message;

	/** Holds a delegate that is executed when a file was picked. */
	FOnDirectoryChanged OnDirectoryChanged;

	TSharedPtr<SEditableTextBox> EditableTextBox;
};

#undef UE_API
