// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Attribute.h"
#include "Fonts/SlateFontInfo.h"
#include "Input/Reply.h"
#include "Styling/SlateWidgetStyleAsset.h"
#include "Styling/ISlateStyle.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Styling/SlateTypes.h"

#define UE_API DESKTOPWIDGETS_API

class SEditableTextBox;

/**
 * Declares a delegate that is executed when a file was picked in the SFilePathPicker widget.
 *
 * The first parameter will contain the path to the picked file.
 */
DECLARE_DELEGATE_OneParam(FOnPathPicked, const FString& /*PickedPath*/);


/**
 * Implements an editable text box with a browse button.
 */
class SFilePathPicker
	: public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SFilePathPicker)
		: _BrowseButtonToolTip(NSLOCTEXT("SFilePathPicker", "BrowseButtonToolTip", "Choose a file from this computer"))
		, _FileTypeFilter(TEXT("All files (*.*)|*.*"))
		, _Font()
		, _IsReadOnly(false)
		, _DialogReturnsFullPath(false)
	{ }

		/** Browse button image resource. */
		SLATE_ATTRIBUTE(const FSlateBrush*, BrowseButtonImage)

		/** Browse button visual style. */
		SLATE_STYLE_ARGUMENT(FButtonStyle, BrowseButtonStyle)

		/** Browse button tool tip text. */
		SLATE_ATTRIBUTE(FText, BrowseButtonToolTip)

		/** The directory to browse by default */
		SLATE_ATTRIBUTE(FString, BrowseDirectory)

		/** Title for the browse dialog window. */
		SLATE_ATTRIBUTE(FText, BrowseTitle)

		/** The currently selected file path. */
		SLATE_ATTRIBUTE(FString, FilePath)

		/** File type filter string. */
		SLATE_ATTRIBUTE(FString, FileTypeFilter)

		/** Font color and opacity of the path text box. */
		SLATE_ATTRIBUTE(FSlateFontInfo, Font)

		/** Whether the path text box can be modified by the user. */
		SLATE_ATTRIBUTE(bool, IsReadOnly)

		/** Whether the path returned by the dialog should be converted from relative to full */
		SLATE_ATTRIBUTE(bool, DialogReturnsFullPath)

		/** Called when a file path has been picked. */
		SLATE_EVENT(FOnPathPicked, OnPathPicked)

	SLATE_END_ARGS()

	/**
	 * Constructs a new widget.
	 *
	 * @param InArgs The construction arguments.
	 */
	UE_API void Construct( const FArguments& InArgs );

	/** Method which simulates the text input in the path text box. */
	UE_API void SimulateTextInput(const FText& NewText);

private:

	/** Callback for clicking the browse button. */
	UE_API FReply HandleBrowseButtonClicked( );

	/** Callback for getting the text in the path text box. */
	UE_API FText HandleTextBoxText( ) const;

	/** Callback for committing the text in the path text box. */
	UE_API void HandleTextBoxTextCommitted( const FText& NewText, ETextCommit::Type /*CommitInfo*/ );

private:

	/** Holds the directory path to browse by default. */
	TAttribute<FString> BrowseDirectory;

	/** Holds the title for the browse dialog window. */
	TAttribute<FText> BrowseTitle;

	/** Holds the currently selected file path. */
	TAttribute<FString> FilePath;

	/** Holds the file type filter string. */
	TAttribute<FString> FileTypeFilter;

	/** Holds the editable text box. */
	TSharedPtr<SEditableTextBox> TextBox;

	/** Holds the option for the dialog to return full path instead of relative. */
	TAttribute<bool> DialogReturnsFullPath;

private:

	/** Holds a delegate that is executed when a file was picked. */
	FOnPathPicked OnPathPicked;
};

#undef UE_API
